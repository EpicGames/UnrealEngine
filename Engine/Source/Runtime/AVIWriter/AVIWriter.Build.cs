// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AVIWriter : ModuleRules
	{
		public AVIWriter(ReadOnlyTargetRules Target) : base(Target)
		{
			bRequiresPlatformSDK = true;

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
				}
				);

			if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
			{
				if (Target.bBuildDeveloperTools)
				{
					AddEngineThirdPartyPrivateStaticDependencies(Target, "DirectShow");
				}
			}
			else if (Target.Platform == UnrealTargetPlatform.Mac)
			{
				PublicFrameworks.AddRange(new string[] { "AVFoundation", "CoreVideo", "CoreMedia" });
			}
		}
	}
}
