// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AVIWriter.cpp: AVI creation implementation.
=============================================================================*/
#include "AVIWriter.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/FileManager.h"
#include "Misc/ScopeLock.h"
#include "Async/Async.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY_STATIC(LogAVIWriter, Log, All);

class FAVIWriterModule : public IModuleInterface
{
};

IMPLEMENT_MODULE(FAVIWriterModule, AVIWriter);

#if PLATFORM_WINDOWS && WITH_UNREAL_DEVELOPER_TOOLS

#include "Windows/AllowWindowsPlatformTypes.h"

typedef TCHAR* PTCHAR;

THIRD_PARTY_INCLUDES_START
#if !defined(__clang__)
	#pragma warning(push)
	#pragma warning(disable: 4263) // 'function' : member function does not override any base class virtual member function
	#pragma warning(disable: 4264) // 'virtual_function' : no override available for virtual member function from base 'cla
	#if USING_CODE_ANALYSIS
		#pragma warning(disable: 6509)  // Invalid annotation: 'return' cannot be referenced in some contexts
		#pragma warning(disable: 6101)  // Returning uninitialized memory '*lpdwExitCode'.  A successful path through the function does not set the named _Out_ parameter.
		#pragma warning(disable: 28204) // 'Func' has an override at 'file' and only the override is annotated for _Param_(N): when an override is annotated, the base (this function) should be similarly annotated.
	#endif // USING_CODE_ANALYSIS
#endif // !defined(__clang__)
#include <streams.h>
#include <dshow.h>
#include <initguid.h>
#if !defined(__clang__)
	#pragma warning(pop)
#endif // !defined(__clang__)
THIRD_PARTY_INCLUDES_END

#include "Windows/HideWindowsPlatformTypes.h"

#include "CapturePin.h"
#include "CaptureSource.h"


#define g_wszCapture    L"Capture Filter"

// Filter setup data
const AMOVIESETUP_MEDIATYPE sudOpPinTypes =
{
	&MEDIATYPE_Video,       // Major type
	&MEDIASUBTYPE_NULL      // Minor type
};


#ifdef __clang__
	// Suppress warning about filling non-const string variable from literal
	#pragma clang diagnostic push
	#pragma clang diagnostic ignored "-Wwritable-strings"	// warning : ISO C++11 does not allow conversion from string literal to 'LPWSTR' (aka 'wchar_t *') [-Wwritable-strings]
#endif

const AMOVIESETUP_PIN sudOutputPinDesktop = 
{
	(LPWSTR)L"Output",      // Obsolete, not used.
	false,          // Is this pin rendered?
	true,           // Is it an output pin?
	false,          // Can the filter create zero instances?
	false,          // Does the filter create multiple instances?
	&CLSID_NULL,    // Obsolete.
	NULL,           // Obsolete.
	1,              // Number of media types.
	&sudOpPinTypes  // Pointer to media types.
};

#ifdef __clang__
	#pragma clang diagnostic pop
#endif

const AMOVIESETUP_FILTER sudPushSourceDesktop =
{
	&CLSID_ViewportCaptureSource,	// Filter CLSID
	g_wszCapture,			// String name
	MERIT_DO_NOT_USE,       // Filter merit
	1,                      // Number pins
	&sudOutputPinDesktop    // Pin details
};

CFactoryTemplate g_Templates[1] = 
{
	{ 
		g_wszCapture,					// Name
		nullptr,						// CLSID
		nullptr, 						// Method to create an instance of MyComponent
		nullptr,                        // Initialization function
		&sudPushSourceDesktop           // Set-up information (for filters)
	},
};
int32 g_cTemplates = 0;

/** Find a pin on the specified filter that matches the specified direction */
IPin* GetPin(IBaseFilter* Filter, PIN_DIRECTION PinDir)
{
	IEnumPins* pEnumDownFilterPins = nullptr;
	IPin* Pin = nullptr;

	if (FAILED(Filter->EnumPins(&pEnumDownFilterPins)))
	{
		return nullptr;
	}

	while(S_OK == pEnumDownFilterPins->Next(1, &Pin, nullptr) )
	{
		PIN_DIRECTION ThisPinDir;
		if (SUCCEEDED(Pin->QueryDirection(&ThisPinDir)) && PinDir == ThisPinDir)
		{
			pEnumDownFilterPins->Release();
			return Pin;
		}

		Pin->Release();
	}

	pEnumDownFilterPins->Release();	
	return nullptr;
}

IBaseFilter* FindEncodingFilter(const FString& Name)
{
	// Create an encoding filter
	ICreateDevEnum* DeviceDenumerator = nullptr;
	if (FAILED(CoCreateInstance(CLSID_SystemDeviceEnum, NULL, CLSCTX_INPROC, IID_ICreateDevEnum, (void**)&DeviceDenumerator)))
	{
		return nullptr;
	}

	IEnumMoniker* EnumIterator = nullptr;
	if (DeviceDenumerator->CreateClassEnumerator(CLSID_VideoCompressorCategory, &EnumIterator, 0) != S_OK)
	{
		return nullptr; //-V773 - Temporary to avoid side effects
	}

	IMoniker* Moniker = nullptr;
	while (EnumIterator->Next(1, &Moniker, nullptr) == S_OK)
	{
		IPropertyBag* Properties = nullptr;
		CA_SUPPRESS(6387);
		if (FAILED(Moniker->BindToStorage(0, 0, IID_IPropertyBag, (void**)&Properties)))
		{
			Moniker->Release();
			continue;
		}

		bool bUseThisEncoder = false;

		VARIANT varName;
		VariantInit(&varName);
		if (SUCCEEDED(Properties->Read(L"FriendlyName", &varName, 0)) && FCString::Stricmp(*Name, varName.bstrVal) == 0)
		{
			bUseThisEncoder = true;
		}
		VariantClear(&varName);

		Properties->Release();

		IBaseFilter* Filter = nullptr;
		CA_SUPPRESS(6387);
		if (bUseThisEncoder && Moniker->BindToObject(nullptr, nullptr, IID_IBaseFilter, (void**)&Filter) == S_OK)
		{
			Filter->AddRef();
			Moniker->Release();
			return Filter;
		}

		Moniker->Release();
	}

	return nullptr;
}

/**
 * Windows implementation relying on DirectShow.
 */
class FAVIWriterWin : public FAVIWriter
{
public:
	FAVIWriterWin( const FAVIWriterOptions& InOptions ) 
		: FAVIWriter(InOptions)
		, Graph(nullptr)
		, Control(nullptr)
		, Capture(nullptr)
		, CaptureFilter(nullptr)
		, EncodingFilter(nullptr)
	{
	};

public:

	virtual void Initialize() override
	{
		// Initialize the COM library.
		if (!FWindowsPlatformMisc::CoInitialize()) 
		{
			UE_LOGF(LogAVIWriter, Error, "ERROR - Could not initialize COM library!");
			return;
		}

		// Create the filter graph manager and query for interfaces.
		HRESULT hr = CoCreateInstance(CLSID_FilterGraph, NULL, CLSCTX_INPROC_SERVER, IID_IGraphBuilder, (void **)&Graph);
		if (FAILED(hr)) 
		{
			UE_LOGF(LogAVIWriter, Error, "ERROR - Could not create the Filter Graph Manager!");
			FWindowsPlatformMisc::CoUninitialize();
			return;
		}

		// Create the capture graph builder
		hr = CoCreateInstance(CLSID_CaptureGraphBuilder2 , NULL, CLSCTX_INPROC, IID_ICaptureGraphBuilder2, (void **) &Capture);
		if (FAILED(hr)) 
		{
			UE_LOGF(LogAVIWriter, Error, "ERROR - Could not create the Capture Graph Manager!");
			FWindowsPlatformMisc::CoUninitialize();
			return;
		}

		// Specify a filter graph for the capture graph builder to use
		hr = Capture->SetFiltergraph(Graph);
		if (FAILED(hr))
		{
			UE_LOGF(LogAVIWriter, Error, "ERROR - Failed to set capture filter graph!");
			FWindowsPlatformMisc::CoUninitialize();
			return;
		}

		CaptureFilter = new FCaptureSource(*this);
		if (FAILED(hr)) 
		{
			UE_LOGF(LogAVIWriter, Error, "ERROR - Could not create CaptureSource filter!");
			Graph->Release();
			FWindowsPlatformMisc::CoUninitialize(); 
			return;
		}
		CaptureFilter->AddRef();

		hr = Graph->AddFilter(CaptureFilter, L"Capture"); 
		if (FAILED(hr)) 
		{
			UE_LOGF(LogAVIWriter, Error, "ERROR - Could not add CaptureSource filter!");
			CaptureFilter->Release();
			Graph->Release();
			FWindowsPlatformMisc::CoUninitialize(); 
			return;
		}

		if (!Options.CodecName.IsEmpty())
		{
			EncodingFilter = FindEncodingFilter(Options.CodecName);
			if (EncodingFilter)
			{
				EncodingFilter->AddRef();
				Graph->AddFilter( EncodingFilter, TEXT("Encoder") );
			}
			else
			{
				UE_LOGF(LogAVIWriter, Warning, "WARNING - Codec %ls not found", *Options.CodecName);
			}
		}

		if (Options.CompressionQuality.IsSet())
		{
			if (!EncodingFilter)
			{
				// Attempt to use a default encoder
				CA_SUPPRESS(6031);
				CoCreateInstance(CLSID_MJPGEnc, NULL, CLSCTX_INPROC, IID_IBaseFilter, (void**)&EncodingFilter);
				Graph->AddFilter( EncodingFilter, TEXT("Encoder") );
			}

			IAMVideoCompression* CompressionImpl = nullptr;
			if (EncodingFilter)
			{
				IEnumPins* pEnum = nullptr;
				IPin* pPin = nullptr;
				EncodingFilter->EnumPins(&pEnum);
				while (S_OK == pEnum->Next(1, &pPin, NULL))
				{
					hr = pPin->QueryInterface(IID_IAMVideoCompression, (void**)&CompressionImpl);
					pPin->Release();
					if (SUCCEEDED(hr)) // Found the interface.
					{
						break;
					}
				}
				pEnum->Release();
			}

			if (CompressionImpl)
			{
				CompressionImpl->put_Quality(Options.CompressionQuality.GetValue());
				CompressionImpl->Release();
			}
		}
		
		IBaseFilter *pMux;
		CA_SUPPRESS(6031);
		CoCreateInstance(CLSID_AviDest, NULL, CLSCTX_INPROC, IID_IBaseFilter, (void**)&pMux);
		hr = Graph->AddFilter(pMux, TEXT("AVI Mux"));
		if (FAILED(hr)) 
		{
			UE_LOGF(LogAVIWriter, Error, "ERROR - Failed to create AVI Mux!");
			Graph->Release();
			FWindowsPlatformMisc::CoUninitialize(); 
			return;
		}

		IBaseFilter *FileWriter;
		CA_SUPPRESS(6031);
		CoCreateInstance(CLSID_FileWriter, NULL, CLSCTX_INPROC, IID_IBaseFilter, (void**)&FileWriter);
		hr = Graph->AddFilter(FileWriter, TEXT("File Writer"));
		if (FAILED(hr)) 
		{
			UE_LOGF(LogAVIWriter, Error, "ERROR - Failed to create file writer!");
			Graph->Release();
			FWindowsPlatformMisc::CoUninitialize(); 
			return;
		}
		
		IFileSinkFilter* Sink = nullptr;
		if(SUCCEEDED(FileWriter->QueryInterface(IID_IFileSinkFilter, (void **)&Sink)))
		{
			Sink->SetFileName(*Options.OutputFilename, nullptr);
		}

		// Now connect the graph
		if (EncodingFilter)
		{
			hr = Graph->Connect(GetPin(CaptureFilter, PINDIR_OUTPUT), GetPin(EncodingFilter, PINDIR_INPUT));
			if (FAILED(hr))
			{
				UE_LOGF(LogAVIWriter, Error, "ERROR - Failed to connect capture filter to encoding filter! (%d)", hr);
				Graph->Release();
				FWindowsPlatformMisc::CoUninitialize(); 
				return;
			}

			hr = Graph->Connect(GetPin(EncodingFilter, PINDIR_OUTPUT), GetPin(pMux, PINDIR_INPUT));
			if (FAILED(hr))
			{
				UE_LOGF(LogAVIWriter, Error, "ERROR - Failed to connect encoding filter to muxer! (%d)", hr);
				Graph->Release();
				FWindowsPlatformMisc::CoUninitialize(); 
				return;
			}
		}
		else
		{
			hr = Graph->Connect(GetPin(CaptureFilter, PINDIR_OUTPUT), GetPin(pMux, PINDIR_INPUT));
			if (FAILED(hr))
			{
				UE_LOGF(LogAVIWriter, Error, "ERROR - Failed to connect capture filter to muxer! (%d)", hr);
				Graph->Release();
				FWindowsPlatformMisc::CoUninitialize(); 
				return;
			}
		}

		Graph->Connect(GetPin(pMux, PINDIR_OUTPUT), GetPin(FileWriter, PINDIR_INPUT));

		if (SUCCEEDED(Graph->QueryInterface(IID_IMediaControl, (void **)&Control)))
		{
			FString Directory = Options.OutputFilename;
			FString Ext = FPaths::GetExtension(Directory, true);

			int32 FPS = FMath::RoundToInt32(double(Options.CaptureFramerateNumerator) / Options.CaptureFramerateDenominator);

			// Keep 3 seconds worth of frames in memory
			CapturedFrames.Reset(new FCapturedFrames(Directory.LeftChop(Ext.Len()) + TEXT("_tmp"), FPS * 3));

			hr = Control->Run();

			bCapturing = true;
		}
		
		pMux->Release();
	}

	void Finalize()
	{
		if (!bCapturing)
		{
			return;
		}

		// Stop the capture pin first to ensure we have all the frames. This blocks until all frames have been sent downstream.
		CaptureFilter->StopCapturing();
		Control->Stop();

		bCapturing = false;
		FrameNumber = 0;

		SAFE_RELEASE(EncodingFilter);
		SAFE_RELEASE(CaptureFilter);
		SAFE_RELEASE(Capture);
		SAFE_RELEASE(Control);
		SAFE_RELEASE(Graph);
		FWindowsPlatformMisc::CoUninitialize();
	}

	virtual void DropFrames(int32 NumFramesToDrop) override
	{
		FrameNumber += NumFramesToDrop;
	}

private:
	IGraphBuilder* Graph;
	IMediaControl* Control;
	ICaptureGraphBuilder2* Capture;
	FCaptureSource* CaptureFilter;
	IBaseFilter* EncodingFilter;
};

#elif PLATFORM_MAC && WITH_UNREAL_DEVELOPER_TOOLS
#include "Mac/MacSystemIncludes.h"
#include "Apple/ScopeAutoreleasePool.h"

#import <AVFoundation/AVFoundation.h>

/**
 * Mac implementation relying on AVFoundation.
 */
class FAVIWriterMac : public FAVIWriter
{
public:
	FAVIWriterMac( const FAVIWriterOptions& InOptions )
	: FAVIWriter(InOptions)
	, AVFWriterRef(nil)
	, AVFWriterInputRef(nil)
	, AVFPixelBufferAdaptorRef(nil)
	, bShutdownRequested(false)
	{
	};
	
public:
	
	virtual void Initialize() override
	{
		SCOPED_AUTORELEASE_POOL;

		if (!bCapturing)
		{
			// Attempt to make the dir if it doesn't exist.
			TCHAR File[MAX_SPRINTF] = TEXT("");
			IFileManager::Get().MakeDirectory(*FPaths::GetPath(Options.OutputFilename), true);
			FCString::Sprintf( File, TEXT("%s"), *Options.OutputFilename );
			
			NSError *Error = nil;
			
			CFStringRef FilePath = FPlatformString::TCHARToCFString(File);
			CFURLRef FileURL = CFURLCreateWithFileSystemPath(kCFAllocatorDefault, FilePath, kCFURLPOSIXPathStyle, false);
			
			// allocate the writer object with our output file URL
			AVFWriterRef = [[AVAssetWriter alloc] initWithURL:(NSURL*)FileURL fileType:AVFileTypeQuickTimeMovie error:&Error];
			
			CFRelease(FilePath);
			CFRelease(FileURL);
			if (Error) {
				UE_LOGF(LogMovieCapture, Error, " AVAssetWriter initWithURL failed ");
				return;
			}
			
			NSDictionary* VideoSettings = nil;
			if (Options.CompressionQuality.IsSet())
			{
				VideoSettings = [NSDictionary dictionaryWithObjectsAndKeys:
								 AVVideoCodecTypeJPEG, AVVideoCodecKey,
								 [NSDictionary dictionaryWithObjectsAndKeys:[NSNumber numberWithFloat:Options.CompressionQuality.GetValue()], AVVideoQualityKey, nil], AVVideoCompressionPropertiesKey,
								 [NSNumber numberWithInt:Options.Width], AVVideoWidthKey,
								 [NSNumber numberWithInt:Options.Height], AVVideoHeightKey,
								 nil];
			}
			else
			{
				VideoSettings = [NSDictionary dictionaryWithObjectsAndKeys:
								 AVVideoCodecTypeH264, AVVideoCodecKey,
								 [NSNumber numberWithInt:Options.Width], AVVideoWidthKey,
								 [NSNumber numberWithInt:Options.Height], AVVideoHeightKey,
								 nil];
			}
			AVFWriterInputRef = [[AVAssetWriterInput
								  assetWriterInputWithMediaType:AVMediaTypeVideo
								  outputSettings:VideoSettings] retain];
			NSDictionary* BufferAttributes = [NSDictionary dictionaryWithObjectsAndKeys:
											  [NSNumber numberWithInt:kCVPixelFormatType_32BGRA], kCVPixelBufferPixelFormatTypeKey, nil];
			
			AVFPixelBufferAdaptorRef = [[AVAssetWriterInputPixelBufferAdaptor
										 assetWriterInputPixelBufferAdaptorWithAssetWriterInput:AVFWriterInputRef
										 sourcePixelBufferAttributes:BufferAttributes] retain];
			check(AVFWriterInputRef);
			check([AVFWriterRef canAddInput:AVFWriterInputRef]);
			[AVFWriterRef addInput:AVFWriterInputRef];
			
			//Start a session:
			[AVFWriterInputRef setExpectsMediaDataInRealTime:YES];
			[AVFWriterRef startWriting];
			[AVFWriterRef startSessionAtSourceTime:kCMTimeZero];
			
			FString Directory = Options.OutputFilename;
			FString Ext = FPaths::GetExtension(Directory, true);

			int32 FPS = FMath::RoundToInt(double(Options.CaptureFramerateNumerator) / Options.CaptureFramerateDenominator);

			// Keep 3 seconds worth of frames in memory
			CapturedFrames.Reset(new FCapturedFrames(Directory.LeftChop(Ext.Len()) + TEXT("_tmp"), FPS * 3));

			bCapturing = true;
			ThreadTaskFuture = Async(EAsyncExecution::Thread, [this]{	TaskThread(); });
		}
	}
	
	void Finalize()
	{
		if (bCapturing)
		{
			SCOPED_AUTORELEASE_POOL;

			bShutdownRequested = true;
			ThreadTaskFuture.Get();
			[AVFWriterInputRef release];
			[AVFWriterRef release];
			[AVFPixelBufferAdaptorRef release];
			AVFWriterInputRef = nil;
			AVFWriterRef = nil;
			AVFPixelBufferAdaptorRef = nil;

			bCapturing = false;
			FrameNumber = 0;
		}
	}
	
	void TaskThread()
	{
		SCOPED_AUTORELEASE_POOL;
		
		for(;;)
		{
			uint32 WaitTimeMs = 100;
			TArray<FCapturedFrame> PendingFrames = GetFrameData(WaitTimeMs);

			// Capture the frames that we have
			for (auto& CurrentFrame : PendingFrames)
			{
				while(![AVFWriterInputRef isReadyForMoreMediaData])
				{
					FPlatformProcess::Sleep(0.0001f);
				}
				
				CVPixelBufferRef PixelBuffer = NULL;
				CVPixelBufferPoolCreatePixelBuffer (NULL, AVFPixelBufferAdaptorRef.pixelBufferPool, &PixelBuffer);
				if(!PixelBuffer)
				{
					CVPixelBufferCreate(kCFAllocatorDefault, Options.Width, Options.Height, kCVPixelFormatType_32BGRA, NULL, &PixelBuffer);
				}
				check(PixelBuffer);
				
				CVPixelBufferLockBaseAddress(PixelBuffer, 0);
				void* Data = CVPixelBufferGetBaseAddress(PixelBuffer);
				FMemory::Memcpy(Data, CurrentFrame.FrameData.GetData(), CurrentFrame.FrameData.Num()*sizeof(FColor));
				CVPixelBufferUnlockBaseAddress(PixelBuffer, 0);
				CMTime PresentTime = CurrentFrame.FrameIndex > 0 ? CMTimeMake(CurrentFrame.FrameIndex * Options.CaptureFramerateDenominator, Options.CaptureFramerateNumerator) : kCMTimeZero;
				BOOL OK = [AVFPixelBufferAdaptorRef appendPixelBuffer:PixelBuffer withPresentationTime:PresentTime];
				check(OK);
				
				CVPixelBufferRelease(PixelBuffer);

				if (CurrentFrame.FrameProcessedEvent)
				{
					CurrentFrame.FrameProcessedEvent->Trigger();
				}
			}

			if (bShutdownRequested && GetNumOutstandingFrames() == 0)
			{
				break;
			}
		}

		[AVFWriterInputRef markAsFinished];
		// This will finish asynchronously and then destroy the relevant objects.
		// We must wait for this to complete.
		FEvent* Event = FPlatformProcess::GetSynchEventFromPool(true);
		[AVFWriterRef finishWritingWithCompletionHandler:^{
			check(AVFWriterRef.status == AVAssetWriterStatusCompleted);
			Event->Trigger();
		}];
		Event->Wait(~0u);
		FPlatformProcess::ReturnSynchEventToPool(Event);
	}

	virtual void DropFrames(int32 NumFramesToDrop) override
	{
		FrameNumber += NumFramesToDrop;
	}
	
private:
	AVAssetWriter* AVFWriterRef;
	AVAssetWriterInput* AVFWriterInputRef;
	AVAssetWriterInputPixelBufferAdaptor* AVFPixelBufferAdaptorRef;
	FThreadSafeBool bShutdownRequested;
	TFuture<void> ThreadTaskFuture;
};
#endif

FCapturedFrame::FCapturedFrame(double InStartTimeSeconds, double InEndTimeSeconds, uint32 InFrameIndex, TArray<FColor> InFrameData)
	: StartTimeSeconds(InStartTimeSeconds)
	, EndTimeSeconds(InEndTimeSeconds)
	, FrameIndex(InFrameIndex)
	, FrameData(MoveTemp(InFrameData))
	, FrameProcessedEvent(nullptr)
{
}

FCapturedFrame::~FCapturedFrame()
{
}

FCapturedFrames::FCapturedFrames(const FString& InArchiveDirectory, int32 InMaxInMemoryFrames)
	: ArchiveDirectory(InArchiveDirectory)
	, MaxInMemoryFrames(InMaxInMemoryFrames)
{
	FrameReady = FPlatformProcess::GetSynchEventFromPool();

	// Ensure the archive directory doesn't exist
	auto& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	PlatformFile.DeleteDirectoryRecursively(*ArchiveDirectory);

	TotalArchivedFrames = 0;
	InMemoryFrames.Reserve(MaxInMemoryFrames);
}

FCapturedFrames::~FCapturedFrames()
{
	FPlatformProcess::ReturnSynchEventToPool(FrameReady);
	FPlatformFileManager::Get().GetPlatformFile().DeleteDirectoryRecursively(*ArchiveDirectory);
}

void FCapturedFrames::Add(FCapturedFrame Frame)
{
	bool bShouldArchive = false;
	{
		FScopeLock Lock(&ArchiveFrameMutex);
		bShouldArchive = ArchivedFrames.Num() != 0;
	}
	
	if (!bShouldArchive)
	{
		FScopeLock Lock(&InMemoryFrameMutex);
		if (InMemoryFrames.Num() < MaxInMemoryFrames)
		{
			InMemoryFrames.Add(MoveTemp(Frame));
		}
		else
		{
			bShouldArchive = true;
		}
	}

	if (bShouldArchive)
	{
		ArchiveFrame(MoveTemp(Frame));
	}
	else
	{
		FrameReady->Trigger();
	}
}

FArchive &operator<<(FArchive& Ar, FCapturedFrame& Frame)
{
	Ar << Frame.StartTimeSeconds;
	Ar << Frame.EndTimeSeconds;
	Ar << Frame.FrameIndex;
	Ar << Frame.FrameData;
	return Ar;
}

void FCapturedFrames::ArchiveFrame(FCapturedFrame Frame)
{
	auto& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	if (!PlatformFile.DirectoryExists(*ArchiveDirectory))
	{
		PlatformFile.CreateDirectory(*ArchiveDirectory);
	}

	// Get (and increment) a unique index for this frame
	uint32 ArchivedFrameIndex = ++TotalArchivedFrames;

	FString Filename = ArchiveDirectory / FString::Printf(TEXT("%d.frame"), ArchivedFrameIndex);
	TUniquePtr<FArchive> Archive(IFileManager::Get().CreateFileWriter(*Filename));
	if (ensure(Archive.IsValid()))
	{
		*Archive << Frame;
		Archive->Close();

		// Add the archived frame to the array
		FScopeLock Lock(&ArchiveFrameMutex);
		ArchivedFrames.Add(ArchivedFrameIndex);
	}
}

TOptional<FCapturedFrame> FCapturedFrames::UnArchiveFrame(uint32 FrameIndex) const
{
	FString Filename = ArchiveDirectory / FString::Printf(TEXT("%d.frame"), FrameIndex);
	TUniquePtr<FArchive> Archive(IFileManager::Get().CreateFileReader(*Filename));
	if (ensure(Archive.IsValid()))
	{
		FCapturedFrame Frame;
		*Archive << Frame;
		Archive->Close();

		FPlatformFileManager::Get().GetPlatformFile().DeleteFile(*Filename);
		return MoveTemp(Frame);
	}

	return TOptional<FCapturedFrame>();
}

void FCapturedFrames::StartUnArchiving()
{
	if (UnarchiveTask.IsSet())
	{
		return;
	}

	UnarchiveTask = Async(EAsyncExecution::Thread, [this]{

		// Attempt to unarchive any archived frames
		ArchiveFrameMutex.Lock();
		TArray<uint32> ArchivedFramesToGet = ArchivedFrames;
		ArchiveFrameMutex.Unlock();

		int32 MaxNumToProcess = FMath::Min(ArchivedFramesToGet.Num(), MaxInMemoryFrames);
		for (int32 Index = 0; Index < MaxNumToProcess; ++Index)
		{
			TOptional<FCapturedFrame> Frame = UnArchiveFrame(ArchivedFramesToGet[Index]);

			if (Frame.IsSet())
			{
				FScopeLock Lock(&InMemoryFrameMutex);
				InMemoryFrames.Add(MoveTemp(Frame.GetValue()));
			}
		}

		if (MaxNumToProcess)
		{
			// Only remove the archived frame indices once we have fully processed them (so that FCapturedFrames::Add knows when to archive frames)
			{
				FScopeLock Lock(&ArchiveFrameMutex);
				ArchivedFrames.RemoveAt(0, MaxNumToProcess, EAllowShrinking::No);
			}

			FrameReady->Trigger();
		}
	});
}

TArray<FCapturedFrame> FCapturedFrames::ReadFrames(uint32 WaitTimeMs)
{
	if (!FrameReady->Wait(WaitTimeMs))
	{
		StartUnArchiving();
		return TArray<FCapturedFrame>();
	}

	UnarchiveTask = TOptional<TFuture<void>>();

	TArray<FCapturedFrame> Frames;
	Frames.Reserve(MaxInMemoryFrames);

	// Swap the frames
	{
		FScopeLock Lock(&InMemoryFrameMutex);
		Swap(Frames, InMemoryFrames);
	}

	StartUnArchiving();

	return Frames;
}

int32 FCapturedFrames::GetNumOutstandingFrames() const
{
	int32 TotalNumFrames = 0;
	{
		FScopeLock Lock(&InMemoryFrameMutex);
		TotalNumFrames += InMemoryFrames.Num();
	}

	{
		FScopeLock Lock(&ArchiveFrameMutex);
		TotalNumFrames += ArchivedFrames.Num();
	}
	
	return TotalNumFrames;
}

FAVIWriter::~FAVIWriter()
{
}

void FAVIWriter::Update(double FrameTimeSeconds, TArray<FColor> FrameData)
{
	if (bCapturing && FrameData.Num())
	{
		double FrameLength = double(Options.CaptureFramerateDenominator) / Options.CaptureFramerateNumerator;
		double FrameStart = FrameNumber * FrameLength;
		FCapturedFrame Frame(FrameStart, FrameStart + FrameLength, FrameNumber, MoveTemp(FrameData));

		FEvent* SyncEvent = nullptr;
		if (Options.bSynchronizeFrames)
		{
			SyncEvent = FPlatformProcess::GetSynchEventFromPool();
			Frame.FrameProcessedEvent = SyncEvent;
		}

		// Add the frame
		CapturedFrames->Add(MoveTemp(Frame));
		FrameNumber++;

		if (SyncEvent)
		{
			SyncEvent->Wait(MAX_uint32);
			FPlatformProcess::ReturnSynchEventToPool(SyncEvent);
		}
	}
}

FAVIWriter* FAVIWriter::CreateInstance(const FAVIWriterOptions& InOptions)
{
#if PLATFORM_WINDOWS && WITH_UNREAL_DEVELOPER_TOOLS
	return new FAVIWriterWin(InOptions);
#elif PLATFORM_MAC && WITH_UNREAL_DEVELOPER_TOOLS
	return new FAVIWriterMac(InOptions);
#else
	return nullptr;
#endif
}
