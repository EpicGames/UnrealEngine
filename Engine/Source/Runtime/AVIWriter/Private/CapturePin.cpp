// Copyright Epic Games, Inc. All Rights Reserved.

#include "CapturePin.h"
#include "AVIWriter.h"
#include "Misc/ScopeExit.h"
#include "CaptureSource.h"

DEFINE_LOG_CATEGORY(LogMovieCapture);

#if PLATFORM_WINDOWS && WITH_UNREAL_DEVELOPER_TOOLS


FCapturePin::FCapturePin(HRESULT *phr, CSource *pFilter, const FAVIWriter& InWriter)
        : CSourceStream(NAME("Push Source"), phr, pFilter, L"Capture")
		, FrameLength(FMath::TruncToInt64(UNITS / (InWriter.Options.CaptureFramerateNumerator / (double)InWriter.Options.CaptureFramerateDenominator)))
		, Writer(InWriter)
{
	ImageWidth  = Writer.GetWidth();
	ImageHeight = Writer.GetHeight();
}

FCapturePin::~FCapturePin()
{
}


//
// GetMediaType
//
// Prefer 5 formats - 8, 16 (*2), 24 or 32 bits per pixel
//
// Prefered types should be ordered by quality, with zero as highest quality.
// Therefore, iPosition =
//      0    Return a 32bit mediatype
//      1    Return a 24bit mediatype
//      2    Return 16bit RGB565
//      3    Return a 16bit mediatype (rgb555)
//      4    Return 8 bit palettised format
//      >4   Invalid
//
HRESULT FCapturePin::GetMediaType(int32 iPosition, CMediaType *pmt)
{
	CheckPointer(pmt,E_POINTER);
	CAutoLock cAutoLock(m_pFilter->pStateLock());

	if(iPosition < 0)
	{
		return E_INVALIDARG;
	}

	// Have we run off the end of types?
	if(iPosition > 4)
	{
		return VFW_S_NO_MORE_ITEMS;
	}

	VIDEOINFO *pvi = (VIDEOINFO *) pmt->AllocFormatBuffer(sizeof(VIDEOINFO));
	if(NULL == pvi)
	{
		return(E_OUTOFMEMORY);
	}

	// Initialize the VideoInfo structure before configuring its members
	ZeroMemory(pvi, sizeof(VIDEOINFO));

	// We only supply 32bit RGB

	// Since we use RGB888 (the default for 32 bit), there is
	// no reason to use BI_BITFIELDS to specify the RGB
	// masks. Also, not everything supports BI_BITFIELDS
	pvi->bmiHeader.biCompression = BI_RGB;
	pvi->bmiHeader.biBitCount    = 32;

	// Adjust the parameters common to all formats
	pvi->bmiHeader.biSize       = sizeof(BITMAPINFOHEADER);
	pvi->bmiHeader.biWidth      = ImageWidth;
	pvi->bmiHeader.biHeight     = ImageHeight;// * (-1); // negative 1 to flip the image vertically
	pvi->bmiHeader.biPlanes     = 1;
	pvi->bmiHeader.biSizeImage  = GetBitmapSize(&pvi->bmiHeader);
	pvi->bmiHeader.biClrImportant = 0;
	pvi->AvgTimePerFrame = FrameLength;

	SetRectEmpty(&(pvi->rcSource)); // we want the whole image area rendered.
	SetRectEmpty(&(pvi->rcTarget)); // no particular destination rectangle

	pmt->SetType(&MEDIATYPE_Video);
	pmt->SetFormatType(&FORMAT_VideoInfo);
	pmt->SetTemporalCompression(true);

	// Work out the GUID for the subtype from the header info.
	//const GUID SubTypeGUID = GetBitmapSubtype(&pvi->bmiHeader);
	pmt->SetSubtype(&MEDIASUBTYPE_RGB32);
	pmt->SetSampleSize(pvi->bmiHeader.biSizeImage);

	return NOERROR;

} // GetMediaType


//
// CheckMediaType
//
// We will accept 8, 16, 24 or 32 bit video formats, in any
// image size that gives room to bounce.
// Returns E_INVALIDARG if the mediatype is not acceptable
//
HRESULT FCapturePin::CheckMediaType(const CMediaType *pMediaType)
{
	CheckPointer(pMediaType,E_POINTER);

	if((*(pMediaType->Type()) != MEDIATYPE_Video) ||   // we only output video
		!(pMediaType->IsFixedSize()))                  // in fixed size samples
	{                                                  
		return E_INVALIDARG;
	}

	// Check for the subtypes we support
	const GUID *SubType = pMediaType->Subtype();
	if (SubType == NULL)
	{
		return E_INVALIDARG;
	}

	if (*SubType != MEDIASUBTYPE_RGB32)
	{
		return E_INVALIDARG;
	}

	// Get the format area of the media type
	VIDEOINFO *pvi = (VIDEOINFO *) pMediaType->Format();

	if(pvi == NULL)
	{
		return E_INVALIDARG;
	}

	// Check if the image width & height have changed
	if(    pvi->bmiHeader.biWidth   != ImageWidth || 
		abs(pvi->bmiHeader.biHeight) != ImageHeight)
	{
		// If the image width/height is changed, fail CheckMediaType() to force
		// the renderer to resize the image.
		return E_INVALIDARG;
	}

	return S_OK;  // This format is acceptable.

} // CheckMediaType


//
// DecideBufferSize
//
// This will always be called after the format has been successfully
// negotiated. So we have a look at m_mt to see what size image we agreed.
// Then we can ask for buffers of the correct size to contain them.
//
HRESULT FCapturePin::DecideBufferSize(IMemAllocator *pAlloc, ALLOCATOR_PROPERTIES *pProperties)
{
	CheckPointer(pAlloc,E_POINTER);
	CheckPointer(pProperties,E_POINTER);

	CAutoLock cAutoLock(m_pFilter->pStateLock());
	HRESULT hr = NOERROR;

	VIDEOINFO *pvi = (VIDEOINFO *) m_mt.Format();
	pProperties->cBuffers = 1;
	pProperties->cbBuffer = pvi->bmiHeader.biSizeImage;

	check(pProperties->cbBuffer);

	// Ask the allocator to reserve us some sample memory. NOTE: the function
	// can succeed (return NOERROR) but still not have allocated the
	// memory that we requested, so we must check we got whatever we wanted.
	ALLOCATOR_PROPERTIES Actual;
	hr = pAlloc->SetProperties(pProperties,&Actual);
	if(FAILED(hr))
	{
		return hr;
	}

	// Is this allocator unsuitable?
	if(Actual.cbBuffer < pProperties->cbBuffer)
	{
		return E_FAIL;
	}

	check(Actual.cBuffers == 1);
	return NOERROR;

} // DecideBufferSize


//
// SetMediaType
//
// Called when a media type is agreed between filters
//
HRESULT FCapturePin::SetMediaType(const CMediaType *pMediaType)
{
	CAutoLock cAutoLock(m_pFilter->pStateLock());

	// Pass the call up to my base class
	HRESULT hr = CSourceStream::SetMediaType(pMediaType);

	if(SUCCEEDED(hr))
	{
		VIDEOINFO * pvi = (VIDEOINFO *) m_mt.Format();
		if (pvi == NULL)
		{
			return E_UNEXPECTED;
		}

		if (pvi->bmiHeader.biBitCount == 32)
		{
			return S_OK;
		}
	}

	// We should never agree any other media types
	check(false);
	return E_INVALIDARG;

} // SetMediaType


// This is where we insert the DIB bits into the video stream.
// FillBuffer is called once for every sample in the stream.
HRESULT FCapturePin::FillBuffer(IMediaSample *pSample)
{
	uint8 *pData;
	long cbData;

	CheckPointer(pSample, E_POINTER);

	CAutoLock cAutoLockShared(&SharedState);

	// Access the sample's data buffer
	pSample->GetPointer(&pData);
	cbData = pSample->GetSize();

	// Check that we're still using video
	check(m_mt.formattype == FORMAT_VideoInfo);

	VIDEOINFOHEADER *pVih = (VIDEOINFOHEADER*)m_mt.pbFormat;

	if (pData)
	{
		uint32 BytesPerRow = sizeof(FColor) * ImageWidth;

		// Fill buffer bottom up
		for (int32 Row = ImageHeight - 1; Row >= 0; --Row, pData += BytesPerRow)
		{
			const FColor* Read = &CurrentFrame->FrameData.GetData()[ImageWidth * Row];
			FMemory::Memcpy(pData, Read, BytesPerRow);
		}
	}

	// Set the timestamps that will govern playback frame rate.
	// The current time is the sample's start.

	// Not strictly necessary since AVI is constant framerate
	REFERENCE_TIME StartTime = FMath::TruncToInt64(UNITS * CurrentFrame->StartTimeSeconds);
	REFERENCE_TIME StopTime = FMath::TruncToInt64(UNITS * CurrentFrame->EndTimeSeconds);
	pSample->SetTime(&StartTime, &StopTime);

	REFERENCE_TIME StartMediaTime = CurrentFrame->FrameIndex;
	REFERENCE_TIME StopMediaTime  = CurrentFrame->FrameIndex+1;
	pSample->SetMediaTime(&StartMediaTime, &StopMediaTime);

	pSample->SetSyncPoint(true);

	return S_OK;
}

// the loop executed while running
HRESULT FCapturePin::DoBufferProcessingLoop(void) 
{
	ON_SCOPE_EXIT
	{
		static_cast<FCaptureSource*>(m_pFilter)->OnFinishedCapturing();
	};

	OnThreadStartPlay();

	bool bPaused = false, bShutdownRequested = false;
	for (;;)
	{
		Command com;
		if (CheckRequest(&com))
		{
			switch(com)
			{
			case CMD_RUN:
			case CMD_PAUSE:
				bPaused = com == CMD_PAUSE;
				Reply(NOERROR);
				break;

			case CMD_STOP:
				break;

			default:
				Reply((uint32) E_UNEXPECTED);
				break;
			}
		}

		bShutdownRequested = bShutdownRequested || !static_cast<FCaptureSource*>(m_pFilter)->ShouldCapture();

		if (!bPaused)
		{
			TOptional<HRESULT> Result = ProcessFrames();
			if (Result.IsSet())
			{
				return Result.GetValue();
			}
		}

		if (bShutdownRequested && 
			(bPaused || Writer.GetNumOutstandingFrames() == 0))
		{
			break;
		}
	}

	return S_FALSE;
}

TOptional<HRESULT> FCapturePin::ProcessFrames()
{
	uint32 WaitTimeMs = 100;
	TArray<FCapturedFrame> PendingFrames = Writer.GetFrameData(WaitTimeMs);

	// Capture the frames that we have
	for (int32 FrameIndex = 0; FrameIndex < PendingFrames.Num(); ++FrameIndex)
	{
		CurrentFrame = &PendingFrames[FrameIndex];
		IMediaSample *pSample = nullptr;

		HRESULT hr;
		hr = GetDeliveryBuffer(&pSample,NULL,NULL,0);
		if (FAILED(hr))
		{
			UE_LOGF(LogMovieCapture, Warning, "Failed to get delivery buffer: %08x; Stopping.", hr);
			return S_OK;
		}

		hr = FillBuffer(pSample);
		if (FAILED(hr))
		{
			pSample->Release();
			UE_LOGF(LogMovieCapture, Warning, "Failed to fill sample buffer: %08x; Stopping.", hr);
			return S_OK;
		}

		hr = Deliver(pSample);
		pSample->Release();
		// downstream filter returns S_FALSE if it wants us to
		// stop or an error if it's reporting an error.
		if(hr != S_OK)
		{
			UE_LOGF(LogMovieCapture, Warning, "Deliver() returned %08x; Stopping.", hr);
			return S_OK;
		}

		if (CurrentFrame->FrameProcessedEvent)
		{
			CurrentFrame->FrameProcessedEvent->Trigger();
		}
	}

	return TOptional<HRESULT>();
}
#endif //#if PLATFORM_WINDOWS
