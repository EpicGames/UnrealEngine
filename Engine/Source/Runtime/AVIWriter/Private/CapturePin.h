// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	CapturePin.h: CapturePin definition
=============================================================================*/

#pragma once

#include "CoreMinimal.h"

class FAVIWriter;
struct FCapturedFrame;

#if PLATFORM_WINDOWS && WITH_UNREAL_DEVELOPER_TOOLS

#include "Windows/AllowWindowsPlatformTypes.h"

typedef TCHAR* PTCHAR;

THIRD_PARTY_INCLUDES_START
#if defined(__clang__)
	#pragma clang diagnostic push
	#pragma clang diagnostic ignored "-Woverloaded-virtual"	 // note: hidden overloaded virtual function 'x' declared here: different number of parameters
#else
	#pragma warning(push)
	#pragma warning(disable: 4263) // 'function' : member function does not override any base class virtual member function
	#pragma warning(disable: 4264) // 'virtual_function' : no override available for virtual member function from base 
	#if USING_CODE_ANALYSIS
		#pragma warning(disable: 6101)  // Returning uninitialized memory '*lpdwExitCode'.  A successful path through the function does not set the named _Out_ parameter.
		#pragma warning(disable: 6509)  // Invalid annotation: 'return' cannot be referenced in some contexts
		#pragma warning(disable: 28204) // 'Func' has an override at 'file' and only the override is annotated for _Param_(N): when an override is annotated, the base (this function) should be similarly annotated.
	#endif // USING_CODE_ANALYSIS
#endif // defined(__clang__)
#include <streams.h>
#if defined(__clang__)
	#pragma clang diagnostic pop
#else
	#pragma warning(pop)
#endif // defined(__clang__)
THIRD_PARTY_INCLUDES_END

#include "Windows/HideWindowsPlatformTypes.h"

class FAVIWriter;
struct FCapturedFrame;

class FCapturePin : public CSourceStream
{
protected:

	/** The length of the frame, used for playback */
	const REFERENCE_TIME FrameLength;
	/** The current image height */
	int32 ImageHeight;
	/** And current image width */
	int32 ImageWidth;
	/** Protects our internal state */
	CCritSec SharedState;

	/** The writer to which we belong */
	const FAVIWriter& Writer;

	/** The frame we're currently processing */
	const FCapturedFrame* CurrentFrame;

	TOptional<HRESULT> ProcessFrames();

public:

	FCapturePin(HRESULT *phr, CSource *pFilter, const FAVIWriter& InWriter);
	~FCapturePin();

	// Override the version that offers exactly one media type
	HRESULT DecideBufferSize(IMemAllocator *pAlloc, ALLOCATOR_PROPERTIES *pRequest);
	HRESULT FillBuffer(IMediaSample *pSample);

	// the loop executed while running
	HRESULT DoBufferProcessingLoop(void);

	// Set the agreed media type and set up the necessary parameters
	HRESULT SetMediaType(const CMediaType *pMediaType);

	// Support multiple display formats
	HRESULT CheckMediaType(const CMediaType *pMediaType);
	HRESULT GetMediaType(int32 iPosition, CMediaType *pmt);

	// Quality control
	// Not implemented because we aren't going in real time.
	// If the file-writing filter slows the graph down, we just do nothing, which means
	// wait until we're unblocked. No frames are ever dropped.
	STDMETHODIMP Notify(IBaseFilter *pSelf, Quality q)
	{
		return E_FAIL;
	}

};
#endif //#if PLATFORM_WINDOWS


