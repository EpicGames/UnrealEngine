// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	CaptureSource.h: CaptureSource implementation
=============================================================================*/

#include "CaptureSource.h"
#include "CapturePin.h"
#include "HAL/PlatformProcess.h"
#include "HAL/Event.h"

#if PLATFORM_WINDOWS && WITH_UNREAL_DEVELOPER_TOOLS

FCaptureSource::FCaptureSource(const FAVIWriter& Writer)
           : CSource(NAME("ViewportCaptureFilter"), nullptr, CLSID_ViewportCaptureSource)
{
	HRESULT hr;
	new FCapturePin(&hr, this, Writer);

	ShutdownEvent = FPlatformProcess::GetSynchEventFromPool();
	bShutdownRequested = false;
}

FCaptureSource::~FCaptureSource()
{
	FPlatformProcess::ReturnSynchEventToPool(ShutdownEvent);
}

void FCaptureSource::StopCapturing()
{
	bShutdownRequested = true;
	ShutdownEvent->Wait(~0);
}

void FCaptureSource::OnFinishedCapturing()
{
	ShutdownEvent->Trigger();
}

#endif //#if PLATFORM_WINDOWS
