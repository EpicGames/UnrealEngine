// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	CaptureSource.h: CaptureSource definition
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "HAL/ThreadSafeBool.h"

class FAVIWriter;

#if PLATFORM_WINDOWS && WITH_UNREAL_DEVELOPER_TOOLS

typedef TCHAR* PTCHAR;

#pragma warning(push)
#pragma warning(disable : 4263) // 'function' : member function does not override any base class virtual member function
#pragma warning(disable : 4264) // 'virtual_function' : no override available for virtual member function from base 
#if USING_CODE_ANALYSIS
	#pragma warning(disable:6509) // Invalid annotation: 'return' cannot be referenced in some contexts
	#pragma warning(disable:6101) // Returning uninitialized memory '*lpdwExitCode'.  A successful path through the function does not set the named _Out_ parameter.
	#pragma warning(disable:28204) // 'Func' has an override at 'file' and only the override is annotated for _Param_(N): when an override is annotated, the base (this function) should be similarly annotated.
#endif
#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <streams.h>
THIRD_PARTY_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"
#pragma warning(pop)
class FCapturePin;
class FAVIWriter;

// {F817F8A7-DE00-42CF-826A-7A5654602D8E}
DEFINE_GUID(CLSID_ViewportCaptureSource, 
	0xf817f8a7, 0xde00, 0x42cf, 0x82, 0x6a, 0x7a, 0x56, 0x54, 0x60, 0x2d, 0x8e);


class FCaptureSource : public CSource
{
public:
	FCaptureSource(const FAVIWriter& Writer);
	~FCaptureSource();

	void StopCapturing();
	void OnFinishedCapturing();
	bool ShouldCapture() const { return !bShutdownRequested; }

private:
	FEvent* ShutdownEvent;
	FThreadSafeBool bShutdownRequested;
};
#endif //#if PLATFORM_WINDOWS

