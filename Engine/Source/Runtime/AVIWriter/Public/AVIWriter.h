// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AVIWriter.h: Helper class for creating AVI files.
=============================================================================*/

#pragma once

#include "Async/Future.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "HAL/CriticalSection.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "HAL/ThreadSafeBool.h"
#include "Logging/LogMacros.h"
#include "Math/Color.h"
#include "Misc/Optional.h"
#include "Misc/Paths.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"

class FEvent;
template <typename ResultType> class TFuture;

DECLARE_LOG_CATEGORY_EXTERN(LogMovieCapture, Warning, All);

DECLARE_DELEGATE_RetVal_TwoParams(FString, FResolveFileFormat, const FString&, const FString&);

/** Creation options for the AVI writer */
struct FAVIWriterOptions
{
	FAVIWriterOptions()
		: OutputFilename(FPaths::VideoCaptureDir() / TEXT("Capture.avi"))
		, CaptureFramerateNumerator(30)
		, CaptureFramerateDenominator(1)
		, bSynchronizeFrames(false)
		, Width(0)
		, Height(0)
	{}

	/** Output filename */
	FString OutputFilename;

	/** The numerator of the captured video, ie (30/1) will capture at 30 frames per second.*/
	int32 CaptureFramerateNumerator;

	/** The denominator of the captured video, ie (30/1) will capture at 30 frames per second.*/
	int32 CaptureFramerateDenominator;

	/** Optional compression quality, as a value between 0 and 1 */
	TOptional<float> CompressionQuality;

	/** Optional codec to use for compression */
	FString CodecName;

	/** When true, the game thread will block until captured frames have been processed by the avi writer */
	bool bSynchronizeFrames;

	uint32 Width;

	uint32 Height;
};

/** Data structure representing a captured frame */
struct FCapturedFrame
{
	FCapturedFrame()
		: StartTimeSeconds(), EndTimeSeconds(), FrameIndex(), FrameProcessedEvent(nullptr)
	{
	}

	FCapturedFrame(double InStartTimeSeconds, double InEndTimeSeconds, uint32 InFrameIndex, TArray<FColor> InFrameData);
	~FCapturedFrame();

	FCapturedFrame(FCapturedFrame&& In) : StartTimeSeconds(In.StartTimeSeconds), EndTimeSeconds(In.EndTimeSeconds), FrameIndex(In.FrameIndex), FrameData(MoveTemp(In.FrameData)), FrameProcessedEvent(In.FrameProcessedEvent) {}
	FCapturedFrame& operator=(FCapturedFrame&& In) { StartTimeSeconds = In.StartTimeSeconds; EndTimeSeconds = In.EndTimeSeconds; FrameIndex = In.FrameIndex; FrameData = MoveTemp(In.FrameData); FrameProcessedEvent = In.FrameProcessedEvent; return *this; }

	/** The start time of this frame */
	double StartTimeSeconds;
	/** The End time of this frame */
	double EndTimeSeconds;
	/** The frame index of this frame in the stream */
	uint32 FrameIndex;
	/** The frame data itself (empty for a dropped frame) */
	TArray<FColor> FrameData;
	/** Triggered when the frame has been processed */
	FEvent* FrameProcessedEvent;
};

/** Container for managing captured frames. Temporarily archives frames to the file system when capture rate drops. */
struct FCapturedFrames
{
	/** Construct from a directory to place archives in, and a maximum number of frames we can hold in */
	FCapturedFrames(const FString& InArchiveDirectory, int32 InMaxInMemoryFrames);
	~FCapturedFrames();

	/** Add a captured frame to this container. Only to be called from the owner tasread. */
	void Add(FCapturedFrame Frame);

	/** Read frames from this container (potentially from a thread) */
	TArray<FCapturedFrame> ReadFrames(uint32 WaitTimeMs);

	/** Retrieve the number of oustanding frames we have not processed yet */
	int32 GetNumOutstandingFrames() const;

private:

	/** Archive a frame */
	void ArchiveFrame(FCapturedFrame Frame);

	/** Unarchive a frame represented by the given unique index */
	TOptional<FCapturedFrame> UnArchiveFrame(uint32 FrameIndex) const;
	
	/** Start a task to unarchive some frames */
	void StartUnArchiving();

	/** The directory in which we will place temporarily archived frames */
	FString ArchiveDirectory;

	/** A mutex to protect the archived frame indices */
	mutable FCriticalSection ArchiveFrameMutex;
	/** Array of unique frame indices that have been archived */
	TArray<uint32> ArchivedFrames;

	/** The total number of frames that have been archived since capturing started */
	uint32 TotalArchivedFrames;

	/** An event that triggers when there are in-memory frames ready for collection */
	FEvent* FrameReady;

	/** Mutex to protect the in memory frames */
	mutable FCriticalSection InMemoryFrameMutex;
	/** Array of in-memory captured frames */
	TArray<FCapturedFrame> InMemoryFrames;

	/** The maximum number of frames we are to store in memory before archiving */
	int32 MaxInMemoryFrames;

	/** Unarchive task result */
	TOptional<TFuture<void>> UnarchiveTask;
};

/** Class responsible for writing frames out to an AVI file */
class FAVIWriter
{
protected:
	/** Protected constructor to avoid abuse. */
	FAVIWriter(const FAVIWriterOptions& InOptions) 
		: bCapturing(false)
		, FrameNumber(0)
		, Options(InOptions)
	{
	}

	/** Whether we are capturing or not */
	FThreadSafeBool bCapturing;

	/** The current frame number */
	int32 FrameNumber;
	
	/** Container that manages frames that we have already captured */
	mutable TUniquePtr<FCapturedFrames> CapturedFrames;

public:
	
	/** Public destruction */
	AVIWRITER_API virtual ~FAVIWriter();
	
	/** Creation options */
	FAVIWriterOptions Options;

	/** Create a new avi writer from the specified options */
	AVIWRITER_API static FAVIWriter* CreateInstance(const FAVIWriterOptions& InOptions);

	/** Access captured frame data. Safe to be called from any thread. */
	TArray<FCapturedFrame> GetFrameData(uint32 WaitTimeMs) const
	{
		return CapturedFrames.IsValid() ? CapturedFrames->ReadFrames(WaitTimeMs) : TArray<FCapturedFrame>();
	}

	/** Retrieve the number of oustanding frames we have not processed yet */
	int32 GetNumOutstandingFrames() const
	{
		return CapturedFrames.IsValid() ? CapturedFrames->GetNumOutstandingFrames() : 0;
	}

	uint32 GetWidth() const
	{
		return Options.Width;
	}

	uint32 GetHeight() const
	{
		return Options.Height;
	}

	int32 GetFrameNumber() const
	{
		return FrameNumber;
	}

	bool IsCapturing() const
	{
		return bCapturing;
	}

	AVIWRITER_API void Update(double FrameTimeSeconds, TArray<FColor> FrameData);
	
	virtual void Initialize() = 0;
	virtual void Finalize() = 0;
	
	virtual void DropFrames(int32 NumFramesToDrop) = 0;
};
